{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": ["WinGDK"],
	"Modules": [
		{
			"Name": "GDKNetDriver",
			"Type": "Runtime",
			"LoadingPhase":  "Default",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}