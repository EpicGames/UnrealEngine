{
	"FileVersion": 3,

	"FriendlyName" : "GDK Net Driver",
	"Version" : 1,
	"VersionName" : "1.0",
	"Category" : "Online Platform",
	"Description": "Net driver for GDK platforms",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"HasExplicitPlatforms": true,	
	"SupportedTargetPlatforms": ["Win64"],
	"Modules": [
		{
			"Name": "GDKNetDriver",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase":  "Default",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": ["Win64"]
		}
	],
	"LocalizationTargets": [
		{
			"Name": "GDKNetDriver",
			"LoadingPolicy": "Always"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		}
	]
}