// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;
public class GDKNetDriver : ModuleRules
{
	public GDKNetDriver(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Engine",
			"Core",
			"CoreUObject",
			"OnlineSubsystemUtils"
		});
	}
}