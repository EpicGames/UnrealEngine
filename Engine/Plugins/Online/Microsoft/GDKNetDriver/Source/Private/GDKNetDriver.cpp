// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKNetDriver.h"
#include "Misc/ConfigCacheIni.h"

UGDKNetDriver::UGDKNetDriver(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

int UGDKNetDriver::GetClientPort()
{
	// GDK requires clients to use the port that was specified in the networking manifest.
	int Port = 0;
	GConfig->GetInt(TEXT("URL"), TEXT("Port"), Port, GEngineIni);
	return Port;
}
