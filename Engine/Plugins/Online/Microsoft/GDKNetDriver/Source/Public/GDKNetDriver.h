// Copyright Epic Games, Inc. All Rights Reserved.

//
// GDK based implementation of the net driver
//

#pragma once
#include "IpNetDriver.h"
#include "GDKNetDriver.generated.h"


UCLASS( transient, config=Engine )
class UGDKNetDriver : public UIpNetDriver
{
	GENERATED_UCLASS_BODY()

	virtual int GetClientPort() override;
};
